[URL]
GameNameShort=MUSS

[Plugins]
+ProgramEnabledPlugins="UdpMessaging"
+ProgramEnabledPlugins="QuicMessaging"
+ProgramEnabledPlugins="ConcertSyncServer"
+ProgramEnabledPlugins="MultiUserServer"

[/Script/MacTargetPlatform.XcodeProjectSettings]
TemplateMacPlist=(FilePath="/Engine/Source/Programs/UnrealMultiUserSlateServer/Resources/Mac/Info.plist")
